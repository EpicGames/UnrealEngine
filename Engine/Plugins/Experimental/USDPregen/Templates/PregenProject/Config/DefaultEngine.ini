

[/Script/EngineSettings.GameMapsSettings]
GameDefaultMap=None
EditorStartupMap=None

[/Script/Engine.WorldPartitionSettings]
bNewMapsEnableWorldPartitionStreaming=False

[/Script/Engine.RendererSettings]
r.AllowStaticLighting=False
r.RayTracing=False
r.RayTracing.RayTracingProxies.ProjectEnabled=False
r.Shadow.Virtual.Enable=0
r.Lumen.Global.Enable=False
r.DynamicGlobalIlluminationMethod=0
r.ReflectionMethod=0
r.GenerateMeshDistanceFields=False
r.SkinCache.CompileShaders=False
r.DefaultFeature.MotionBlur=False
r.DefaultFeature.LensFlare=False
r.DefaultFeature.AmbientOcclusion=False
r.AntiAliasingMethod=0
r.DefaultFeature.AutoExposure.ExtendDefaultLuminanceRange=False
r.DefaultFeature.LocalExposure.HighlightContrastScale=1.0
r.DefaultFeature.LocalExposure.ShadowContrastScale=1.0

[/Script/Engine.WorldSettings]
bEnableWorldPartition=False
bEnableWorldPartition2D=False

[/Script/Engine.Engine]
bUseFixedFrameRate=True
FixedFrameRate=30
+ActiveGameNameRedirects=(OldGameName="TP_BlankBP",NewGameName="/Script/PregenWorker")
+ActiveGameNameRedirects=(OldGameName="/Script/TP_BlankBP",NewGameName="/Script/PregenWorker")

[/Script/Engine.AudioSettings]
bDisableAudio=True

[ConsoleVariables]
Interchange.FeatureFlags.Import.USD.ToLevel=1
USD.InstancingAwareTranslation=0
Interchange.FeatureFlags.Import.USDPregen.Enable=1

[/Script/WindowsTargetPlatform.WindowsTargetSettings]
DefaultGraphicsRHI=DefaultGraphicsRHI_DX12
DefaultGraphicsRHI=DefaultGraphicsRHI_DX12
-D3D12TargetedShaderFormats=PCD3D_SM5
+D3D12TargetedShaderFormats=PCD3D_SM6
-D3D11TargetedShaderFormats=PCD3D_SM5
+D3D11TargetedShaderFormats=PCD3D_SM5
Compiler=Default
AudioSampleRate=48000
AudioCallbackBufferFrameSize=1024
AudioNumBuffersToEnqueue=1
AudioMaxChannels=0
AudioNumSourceWorkers=0
SpatializationPlugin=
SourceDataOverridePlugin=
ReverbPlugin=
OcclusionPlugin=
CompressionOverrides=(bOverrideCompressionTimes=False,DurationThreshold=5.000000,MaxNumRandomBranches=0,SoundCueQualityIndex=0)
CacheSizeKB=65536
MaxChunkSizeOverrideKB=0
bResampleForDevice=False
MaxSampleRate=48000.000000
HighSampleRate=32000.000000
MedSampleRate=24000.000000
LowSampleRate=12000.000000
MinSampleRate=8000.000000
CompressionQualityModifier=1.000000
AutoStreamingThreshold=0.000000
SoundCueCookQualityIndex=-1

[/Script/LinuxTargetPlatform.LinuxTargetSettings]
-TargetedRHIs=SF_VULKAN_SM5
+TargetedRHIs=SF_VULKAN_SM6

[/Script/MacTargetPlatform.MacTargetSettings]
-TargetedRHIs=SF_METAL_SM5
+TargetedRHIs=SF_METAL_SM6

[/Script/HardwareTargeting.HardwareTargetingSettings]
TargetedHardwareClass=Desktop
AppliedTargetedHardwareClass=Desktop
DefaultGraphicsPerformance=Maximum
AppliedDefaultGraphicsPerformance=Maximum

[/Script/WorldPartitionEditor.WorldPartitionEditorSettings]
CommandletClass=Class'/Script/UnrealEd.WorldPartitionConvertCommandlet'

[/Script/Engine.UserInterfaceSettings]
bAuthorizeAutomaticWidgetVariableCreation=False
FontDPIPreset=Standard
FontDPI=72

[/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings]
bEnablePlugin=True
bAllowNetworkConnection=True
SecurityToken=5EEC739B4E01EC07DD2B95AB464279F0
bIncludeInShipping=False
bAllowExternalStartInShipping=False
bCompileAFSProject=False
bUseCompression=False
bLogFiles=False
bReportStats=False
ConnectionType=USBOnly
bUseManualIPAddress=False
ManualIPAddress=

