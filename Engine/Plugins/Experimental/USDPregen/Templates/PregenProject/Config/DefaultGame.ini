

[/Script/CommonUI.CommonUISettings]
CommonButtonAcceptKeyHandling=TriggerClick

[/Script/EngineSettings.GeneralProjectSettings]
ProjectID=5692DFF34CBBEC79633F0192A96711D0
