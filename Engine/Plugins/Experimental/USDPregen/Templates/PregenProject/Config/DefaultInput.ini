[/Script/Engine.InputSettings]
-AxisConfig=(AxisKeyName="Gamepad_LeftX",AxisProperties=(DeadZone=0.25,Exponent=1.f,Sensitivity=1.f))
-AxisConfig=(AxisKeyName="Gamepad_LeftY",AxisProperties=(DeadZone=0.25,Exponent=1.f,Sensitivity=1.f))
-AxisConfig=(AxisKeyName="Gamepad_RightX",AxisProperties=(DeadZone=0.25,Exponent=1.f,Sensitivity=1.f))
-AxisConfig=(AxisKeyName="Gamepad_RightY",AxisProperties=(DeadZone=0.25,Exponent=1.f,Sensitivity=1.f))
-AxisConfig=(AxisKeyName="MouseX",AxisProperties=(DeadZone=0.f,Exponent=1.f,Sensitivity=0.07f))
-AxisConfig=(AxisKeyName="MouseY",AxisProperties=(DeadZone=0.f,Exponent=1.f,Sensitivity=0.07f))
-AxisConfig=(AxisKeyName="Mouse2D",AxisProperties=(DeadZone=0.f,Exponent=1.f,Sensitivity=0.07f))
+AxisConfig=(AxisKeyName="Gamepad_LeftX",AxisProperties=(DeadZone=0.250000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="Gamepad_LeftY",AxisProperties=(DeadZone=0.250000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="Gamepad_RightX",AxisProperties=(DeadZone=0.250000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="Gamepad_RightY",AxisProperties=(DeadZone=0.250000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="MouseX",AxisProperties=(DeadZone=0.000000,Sensitivity=0.070000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="MouseY",AxisProperties=(DeadZone=0.000000,Sensitivity=0.070000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="Mouse2D",AxisProperties=(DeadZone=0.000000,Sensitivity=0.070000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="MouseWheelAxis",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="Gamepad_LeftTriggerAxis",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="Gamepad_RightTriggerAxis",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="Gamepad_Special_Left_X",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="Gamepad_Special_Left_Y",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="Vive_Left_Trigger_Axis",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="Vive_Left_Trackpad_X",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="Vive_Left_Trackpad_Y",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="Vive_Right_Trigger_Axis",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="Vive_Right_Trackpad_X",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="Vive_Right_Trackpad_Y",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="MixedReality_Left_Trigger_Axis",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="MixedReality_Left_Thumbstick_X",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="MixedReality_Left_Thumbstick_Y",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="MixedReality_Left_Trackpad_X",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="MixedReality_Left_Trackpad_Y",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="MixedReality_Right_Trigger_Axis",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="MixedReality_Right_Thumbstick_X",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="MixedReality_Right_Thumbstick_Y",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="MixedReality_Right_Trackpad_X",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="MixedReality_Right_Trackpad_Y",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="OculusTouch_Left_Grip_Axis",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="OculusTouch_Left_Trigger_Axis",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="OculusTouch_Left_Thumbstick_X",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="OculusTouch_Left_Thumbstick_Y",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="OculusTouch_Right_Grip_Axis",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="OculusTouch_Right_Trigger_Axis",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="OculusTouch_Right_Thumbstick_X",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="OculusTouch_Right_Thumbstick_Y",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="ValveIndex_Left_Grip_Axis",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="ValveIndex_Left_Grip_Force",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="ValveIndex_Left_Trigger_Axis",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="ValveIndex_Left_Thumbstick_X",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="ValveIndex_Left_Thumbstick_Y",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="ValveIndex_Left_Trackpad_X",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="ValveIndex_Left_Trackpad_Y",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="ValveIndex_Left_Trackpad_Force",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="ValveIndex_Right_Grip_Axis",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="ValveIndex_Right_Grip_Force",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="ValveIndex_Right_Trigger_Axis",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="ValveIndex_Right_Thumbstick_X",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="ValveIndex_Right_Thumbstick_Y",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="ValveIndex_Right_Trackpad_X",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="ValveIndex_Right_Trackpad_Y",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
+AxisConfig=(AxisKeyName="ValveIndex_Right_Trackpad_Force",AxisProperties=(DeadZone=0.000000,Sensitivity=1.000000,Exponent=1.000000,bInvert=False))
bAltEnterTogglesFullscreen=True
bF11TogglesFullscreen=True
bUseMouseForTouch=False
bEnableMouseSmoothing=True
bEnableFOVScaling=True
bCaptureMouseOnLaunch=True
bEnableLegacyInputScales=True
bEnableMotionControls=True
bFilterInputByPlatformUser=False
bShouldFlushPressedKeysOnViewportFocusLost=True
bAlwaysShowTouchInterface=False
bShowConsoleOnFourFingerTap=True
bEnableGestureRecognizer=False
bUseAutocorrect=False
DefaultViewportMouseCaptureMode=CapturePermanently_IncludingInitialMouseDown
DefaultViewportMouseLockMode=LockOnCapture
FOVScale=0.011110
DoubleClickTime=0.200000
DefaultPlayerInputClass=/Script/EnhancedInput.EnhancedPlayerInput
DefaultInputComponentClass=/Script/EnhancedInput.EnhancedInputComponent
DefaultTouchInterface=/Engine/MobileResources/HUD/DefaultVirtualJoysticks.DefaultVirtualJoysticks
-ConsoleKeys=Tilde
+ConsoleKeys=Tilde

