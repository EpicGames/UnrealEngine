[/Script/USDPregenInterchange.USDPregenSettings]
ImportContentPath=(Path="/PregenContent")
PackageSubPathTemplate="assets/${$DEFINITION_NAME}/${versions/$DEFINITION_VERSION}/${permutations/id_$PERMUTATION_ID}"
